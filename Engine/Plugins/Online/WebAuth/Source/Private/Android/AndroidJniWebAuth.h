// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJniGameActivity.h"

namespace UE::Jni
{
	struct FWebAuth
	{
		using PartialClass = FGameActivity;
		
		static void JNICALL handleAuthSessionResponse(JNIEnv* env, jobject thiz, jstring redirectURL);
		
		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(handleAuthSessionResponse)
		};
	};
	
	template struct TInitialize<FWebAuth>;
}
#endif
