// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatformWebAuth.h"

#if USE_ANDROID_JNI
#include "Android/AndroidApplication.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidJniWebAuth.h"
#endif

static TAtomic<FAndroidWebAuth*> AndroidWebAuthPtr(nullptr);


// Returns true if the request is made
// NOTE: We are not using SchemeStr on Android, it's hardcoded in <ProjectName>_UPL.xml
bool FAndroidWebAuth::AuthSessionWithURL(const FString &UrlStr, const FString &SchemeStr, const FWebAuthSessionCompleteDelegate& Delegate)
{
	bool bSessionInProgress = false;

#if USE_ANDROID_JNI
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto JUrlStr = FJavaHelper::ToJavaString(Env, UrlStr);
		static jmethodID AndroidThunkJava_StartAuthSession = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_StartAuthSession", "(Ljava/lang/String;)V", false);
		check(AndroidThunkJava_StartAuthSession != NULL);

		AuthSessionCompleteDelegate = Delegate;
		bSessionInProgress = true;

		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, AndroidThunkJava_StartAuthSession, *JUrlStr);
	}
#endif

	return bSessionInProgress;
}

void FAndroidWebAuth::OnAuthSessionComplete(const FString &RedirectURL, bool bHasResponse)
{
	AuthSessionCompleteDelegate.ExecuteIfBound(RedirectURL, bHasResponse);
	AuthSessionCompleteDelegate = nullptr;
}

bool FAndroidWebAuth::SaveCredentials(const FString& IdStr, const FString& TokenStr, const FString& EnvironmentNameStr)
{
#if USE_ANDROID_JNI
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Id = FJavaHelper::ToJavaString(Env, IdStr);
		auto Token = FJavaHelper::ToJavaString(Env, TokenStr);
		auto EnvironmentName = FJavaHelper::ToJavaString(Env, EnvironmentNameStr);

		// if we have nil/empty params, we just delete and leave
		if (IdStr.IsEmpty() || TokenStr.IsEmpty())
		{
			static jmethodID AndroidThunkJava_WebAuthClearCredentials = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_WebAuthClearCredentials", "(Ljava/lang/String;)V", false);
			check(AndroidThunkJava_WebAuthClearCredentials != NULL);

			FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, AndroidThunkJava_WebAuthClearCredentials, *EnvironmentName);
			return true;
		}
		else
		{
			static jmethodID AndroidThunkJava_WebAuthStoreCredentials = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_WebAuthStoreCredentials", "(Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;)V", false);
			check(AndroidThunkJava_WebAuthStoreCredentials != NULL);

			FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, AndroidThunkJava_WebAuthStoreCredentials, *Id, *Token, *EnvironmentName);
			return true;
		}
	}
#endif

	return false;
}

bool FAndroidWebAuth::LoadCredentials(FString& OutIdStr, FString& OutTokenStr, const FString& EnvironmentNameStr)
{
#if USE_ANDROID_JNI
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto EnvironmentName = FJavaHelper::ToJavaString(Env, EnvironmentNameStr);

		static jmethodID AndroidThunkJava_WebAuthGetId = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_WebAuthGetId", "(Ljava/lang/String;)Ljava/lang/String;", false);
		check(AndroidThunkJava_WebAuthGetId != NULL);
		OutIdStr = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, AndroidThunkJava_WebAuthGetId, *EnvironmentName));

		static jmethodID AndroidThunkJava_WebAuthGetToken = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_WebAuthGetToken", "(Ljava/lang/String;)Ljava/lang/String;", false);
		check(AndroidThunkJava_WebAuthGetToken != NULL);
		OutTokenStr = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, AndroidThunkJava_WebAuthGetToken, *EnvironmentName));

		return true;
	}
#endif

	return false;
}

void FAndroidWebAuth::DeleteLoginCookies(const FString& PrefixStr, const FString& SchemeStr, const FString& DomainStr, const FString& PathStr)
{
#if USE_ANDROID_JNI
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Prefix = FJavaHelper::ToJavaString(Env, PrefixStr);
		auto Scheme = FJavaHelper::ToJavaString(Env, SchemeStr);
		auto Domain = FJavaHelper::ToJavaString(Env, DomainStr);
		auto Path = FJavaHelper::ToJavaString(Env, PathStr);

		static jmethodID AndroidThunkJava_WebAuthDeleteLoginCookies = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_WebAuthDeleteLoginCookies", "(Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;)V", false);
		check(AndroidThunkJava_WebAuthDeleteLoginCookies != NULL);
		if (AndroidThunkJava_WebAuthDeleteLoginCookies == NULL)
		{
			// TODO: Error log
			//UE_LOGF(LogWebAuth, Error, "[FAndroidWebAuth::DeleteLoginCookies] AndroidThunkJava_WebAuthDeleteLoginCookies is NULL");
			return;
		}
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, AndroidThunkJava_WebAuthDeleteLoginCookies, *Prefix, *Scheme, *Domain, *Path);
	}
#endif
}

FAndroidWebAuth::FAndroidWebAuth()
	: AuthSessionCompleteDelegate(nullptr)
{
	verify(AndroidWebAuthPtr.Exchange(this) == nullptr);
}

FAndroidWebAuth::~FAndroidWebAuth()
{
	verify(AndroidWebAuthPtr.Exchange(nullptr) == this);
}


#if USE_ANDROID_JNI
// This function is declared in the Java-defined class, GameActivity.java: "public native void handleAuthSessionResponse(String redirectURL);"
// Auto merged from <ProjectName>_UPL.xml
void JNICALL UE::Jni::FWebAuth::handleAuthSessionResponse(JNIEnv* jenv, jobject thiz, jstring redirectURL)
{
	auto RedirectURL = FJavaHelper::FStringFromParam(jenv, redirectURL);

#if !UE_BUILD_SHIPPING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformWebAuth::AuthSessionComplete: RedirectURL=[%s]"), *RedirectURL);
#endif

	FAndroidWebAuth* AndroidWebAuthInterface = AndroidWebAuthPtr;
	if (AndroidWebAuthInterface != nullptr)
	{
		AndroidWebAuthInterface->OnAuthSessionComplete(RedirectURL, true);
	}
}
#endif


IWebAuth* FAndroidPlatformWebAuth::CreatePlatformWebAuth()
{
	return new FAndroidWebAuth();
}

