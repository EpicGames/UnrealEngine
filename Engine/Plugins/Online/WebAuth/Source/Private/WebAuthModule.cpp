// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAuthModule.h"

#include "Modules/ModuleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"

DEFINE_LOG_CATEGORY(LogWebAuth);


// FWebAuthModule

IMPLEMENT_MODULE(FWebAuthModule, WebAuth);

FWebAuthModule* FWebAuthModule::Singleton = nullptr;


void FWebAuthModule::StartupModule()
{
	UE_LOGF(LogWebAuth, Display, "FWebAuthModule::StartupModule: Starting WebAuth Module");
	Singleton = this;

	// This can return null is the module is not available on this platform, check IsAvailable() before use
	WebAuth = FPlatformWebAuth::CreatePlatformWebAuth();
	if (WebAuth == nullptr)
	{
		// platform does not provide web auth
		UE_LOGF(LogWebAuth, Display, "FWebAuthModule::StartupModule: WebAuth Module not implemented on this platform");
	}
}

void FWebAuthModule::ShutdownModule()
{
	UE_LOGF(LogWebAuth, Display, "FWebAuthModule::ShutdownModule:");
	delete WebAuth;	// can be passed NULLs

	WebAuth = nullptr;
	Singleton = nullptr;
}

bool FWebAuthModule::HandleWebAuthCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
// TODO: Support params, but use test case
#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("Session")))
	{
		FString Url = FParse::Token(Cmd, false);
		FString AppScheme = FParse::Token(Cmd, false);
		if (Url.IsEmpty() || AppScheme.IsEmpty())
		{
			UE_LOGF(LogWebAuth, Warning, "usage: WebAuth Session <url> <app scheme>");
		}
		else
		{
			if (IsAvailable())
			{
				UE_LOGF(LogWebAuth, Display, "Starting auth session");

				GetWebAuth().AuthSessionWithURL(
					Url,
					AppScheme, 
					FWebAuthSessionCompleteDelegate::CreateLambda([this](const FString &RedirectURL, bool bHasResponse) {
					UE_LOGF(LogWebAuth, Display, "WebAuth Session RedirectURL=[%ls], bHasResponse=[%d]", *RedirectURL, bHasResponse);
				}));
			}
			else
			{
				UE_LOGF(LogWebAuth, Warning, "WebAuth not available");
			}
		}
	}
#endif // !UE_BUILD_SHIPPING

	return true;
}

bool FWebAuthModule::Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Ignore any execs that don't start with WebAuth
	if (FParse::Command(&Cmd, TEXT("WebAuth")))
	{
		return HandleWebAuthCommand(Cmd, Ar);
	}
	return false;
}

FWebAuthModule& FWebAuthModule::Get()
{
	UE_LOGF(LogWebAuth, Display, "FWebAuthModule::Get:");

	if (Singleton == nullptr)
	{
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FWebAuthModule>("WebAuth");
	}
	check(Singleton != nullptr);
	return *Singleton;
}

