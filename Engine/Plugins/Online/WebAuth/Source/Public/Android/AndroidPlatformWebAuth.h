// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformWebAuth.h"
#include "WebAuth.h"

#define UE_API WEBAUTH_API


/**
 * Android implementation of Web Authentication
 */
class FAndroidWebAuth : public IWebAuth
{
public:
	//~ Begin IWebAuth Interface
	virtual bool AuthSessionWithURL(const FString &UrlStr, const FString &SchemeStr, const FWebAuthSessionCompleteDelegate& Delegate);

	virtual bool SaveCredentials(const FString& IdStr, const FString& TokenStr, const FString& EnvironmentNameStr);
	virtual bool LoadCredentials(FString& OutIdStr, FString& OutTokenStr, const FString& EnvironmentNameStr);

	virtual void DeleteLoginCookies(const FString& PrefixStr, const FString& SchemeStr, const FString& DomainStr, const FString& PathStr);
	//~ End IWebAuth Interface

	void OnAuthSessionComplete(const FString &RedirectURL, bool bHasResponse);

	/**
	 * Constructor
	 */
	FAndroidWebAuth();

	/**
	 * Destructor
	 */
	virtual ~FAndroidWebAuth();

private:
	/** Delegate that will get called once an authentication session completes or for an error condition */
	FWebAuthSessionCompleteDelegate AuthSessionCompleteDelegate;
};


/**
 * Platform specific WebAuth implementations
 */
class FAndroidPlatformWebAuth : public FGenericPlatformWebAuth
{
public:
	static UE_API IWebAuth* CreatePlatformWebAuth();
};


typedef FAndroidPlatformWebAuth FPlatformWebAuth;

#undef UE_API
