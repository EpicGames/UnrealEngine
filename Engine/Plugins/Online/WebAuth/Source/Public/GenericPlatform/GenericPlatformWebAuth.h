// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IWebAuth;

/**
 * Platform specific WebAuth implementations
 * Intended usage is to use FPlatformWebAuth instead of FGenericPlatformWebAuth
 */
class FGenericPlatformWebAuth
{
public:
	/**
	 * Creates a platform-specific WebAuth.
	 *
	 * @return nullptr if no implementation is available
	 */
	static IWebAuth* CreatePlatformWebAuth()
	{
		return nullptr;
	}
};

