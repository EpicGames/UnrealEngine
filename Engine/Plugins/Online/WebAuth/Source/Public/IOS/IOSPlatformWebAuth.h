// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_IOS && !PLATFORM_TVOS

#include "GenericPlatform/GenericPlatformWebAuth.h"
#include "WebAuth.h"


/**
 * IOS implementation of Web Authentication
 */
class FIOSWebAuth : public IWebAuth
{
public:
	//~ Begin IWebAuth Interface
	virtual bool AuthSessionWithURL(const FString &UrlStr, const FString &SchemeStr, const FWebAuthSessionCompleteDelegate& Delegate);

	virtual bool SaveCredentials(const FString& IdStr, const FString& TokenStr, const FString& EnvironmentNameStr);
	virtual bool LoadCredentials(FString& OutIdStr, FString& OutTokenStr, const FString& EnvironmentNameStr);

	virtual void DeleteLoginCookies(const FString& PrefixStr, const FString& SchemeStr, const FString& DomainStr, const FString& PathStr);
	//~ End IWebAuth Interface

	/**
	 * Constructor
	 */
	FIOSWebAuth();

	/**
	 * Destructor
	 */
	virtual ~FIOSWebAuth();

private:
	/** Delegate handle to get auth response from external browser */
	FDelegateHandle OpenUrlHandle;
	/** Delegate that will get called once an authentication session completes or for an error condition */
	FWebAuthSessionCompleteDelegate AuthSessionCompleteDelegate;
};


/**
 * Platform specific WebAuth implementations
 */
class WEBAUTH_API FIOSPlatformWebAuth : public FGenericPlatformWebAuth
{
public:
	static IWebAuth* CreatePlatformWebAuth();
};


typedef FIOSPlatformWebAuth FPlatformWebAuth;

#endif
