// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IWebAuth;

/**
 * Null implementation of Web Authentication
 */
class FNullPlatformWebAuth
{
public:
	/**
	 * Creates a platform-specific WebAuth.
	 *
	 * @return nullptr if no implementation is available
	 */
	static IWebAuth* CreatePlatformWebAuth()
	{
		return nullptr;
	}
};


typedef FNullPlatformWebAuth FPlatformWebAuth;

