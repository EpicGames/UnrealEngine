// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if  PLATFORM_IOS && !PLATFORM_TVOS
#include "IOS/IOSPlatformWebAuth.h"
#elif PLATFORM_ANDROID
#include "Android/AndroidPlatformWebAuth.h"
#else
#include "NullPlatformWebAuth.h"
#endif

