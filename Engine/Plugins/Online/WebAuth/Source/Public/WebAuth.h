// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Module dependencies

#include "CoreMinimal.h"

/**
 * Delegate called when an authentication session completes
 *
 * @param RedirectURL result received from authenication session attempt
 * @param bHasResponse - true if the request was made, false if we did not proceed
 */
DECLARE_DELEGATE_TwoParams(FWebAuthSessionCompleteDelegate, const FString &/*RedirectURL*/, bool /*bHasResponse*/);

class IWebAuth
{
public:
	virtual bool AuthSessionWithURL(const FString &UrlStr, const FString &SchemeStr, const FWebAuthSessionCompleteDelegate& Delegate) = 0;

	virtual bool SaveCredentials(const FString& IdStr, const FString& TokenStr, const FString& EnvironmentNameStr) = 0;
	virtual bool LoadCredentials(FString& OutIdStr, FString& OutTokenStr, const FString& EnvironmentNameStr) = 0;

	virtual void DeleteLoginCookies(const FString& PrefixStr, const FString& SchemeStr, const FString& DomainStr, const FString& PathStr) = 0;

	/**
	 * Destructor for overrides
	 */
	virtual ~IWebAuth() = default;
};

