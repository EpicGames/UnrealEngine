// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Module dependencies
#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleInterface.h"

// Module includes
#include "PlatformWebAuth.h"
#include "WebAuth.h"

WEBAUTH_API DECLARE_LOG_CATEGORY_EXTERN(LogWebAuth, Log, All);


/**
 * Module for Web Authentication implementation
 */
class FWebAuthModule :
	public IModuleInterface, public FSelfRegisteringExec
{

public:

	/**
	 * Exec command handlers
	 */
	bool HandleWebAuthCommand(const TCHAR* Cmd, FOutputDevice& Ar);

	// FWebAuthModule

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	WEBAUTH_API static FWebAuthModule& Get();

	/**
	 * Get the WebAuth interface
	 *
	 * @return Web authentication request manager used by the module
	 */
	inline IWebAuth& GetWebAuth()
	{
		check(WebAuth != nullptr);
		return *WebAuth;
	}

	/**
	 * Is the WebAuth module available on the active platform?
	 *
	 * @return true if the WebAuth module is supported
	 */

	inline bool IsAvailable()
	{
		return WebAuth != nullptr;
	}

protected:
	// FSelfRegisteringExec

	/**
	 * Handle exec commands starting with "WebAuth"
	 *
	 * @param InWorld	the world context
	 * @param Cmd		the exec command being executed
	 * @param Ar		the archive to log results to
	 *
	 * @return true if the handler consumed the input, false to continue searching handlers
	 */
	virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:
	// IModuleInterface

	/**
	 * Called when WebAuth module is loaded
	 * load dependant modules
	 */
	virtual void StartupModule() override;

	/**
	 * Called when WebAuth module is unloaded
	 */
	virtual void ShutdownModule() override;

	IWebAuth* WebAuth;

	/** singleton for the module while loaded and available */
	static FWebAuthModule* Singleton;
};

