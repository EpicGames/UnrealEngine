// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class WebAuth : ModuleRules
{
	public WebAuth(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		bRequiresPlatformSDK = true;
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PublicDefinitions.Add("WITH_WEBAUTH=1");

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PrivateDependencyModuleNames.Add("ApplicationCore");
			PublicFrameworks.Add("AuthenticationServices");
		}

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateDependencyModuleNames.Add("ApplicationCore");
			PrivateDependencyModuleNames.Add("Launch");

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "WebAuth_UPL.xml"));
		}
	}
}
