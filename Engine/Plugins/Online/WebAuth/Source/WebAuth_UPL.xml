<?xml version="1.0" encoding="utf-8"?>
<!--BattleBreakers additions for Android-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
  <init>
    <log text="WebAuth init"/>
	<setBoolFromProperty result="bWebAuthSetLaunchAuthTab" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bWebAuthSetLaunchAuthTab" default="false"/>
  </init>
  
    <buildGradleAdditions>
         <insert>
 dependencies {
     implementation('com.android.support:customtabs:25.2.0')
 }
         </insert>
     </buildGradleAdditions>
	 
	<gameActivityImportAdditions>
		<insert>
import android.net.Uri;
import androidx.browser.customtabs.CustomTabsIntent;
import static android.content.Intent.ACTION_VIEW;
import android.content.SharedPreferences;
import android.webkit.CookieManager;
import java.net.HttpCookie;
		</insert>
	</gameActivityImportAdditions>

	<gameActivityOnResumeAdditions>
		<insert>
	// Begin WebAuth onResume
	if (WebAuthAwaitingResponse)
	{
		Intent webAuthIntent = getIntent();
		if (bHasNewIntent &amp;&amp; ACTION_VIEW.equals(webAuthIntent.getAction()))
		{
			Uri data = webAuthIntent.getData();
			handleAuthenticationUri(data);
		}
		else if (hasWindowFocus())
		{
			handleAuthenticationUri(null);
		}
	}
	// End WebAuth onResume
		</insert>
	</gameActivityOnResumeAdditions>

	<gameActivityClassAdditions>
		<insert>
			<![CDATA[
	private boolean WebAuthAwaitingResponse = false;

	public native void handleAuthSessionResponse(String redirectURL);

	public void AndroidThunkJava_StartAuthSession(String uri)
	{
		SharedPreferences preferences = getSharedPreferences("WebAuth", Context.MODE_PRIVATE);

		boolean bUseWebAuthAutoUseIntentDataOnFirstTry = "true".equals(ConfigRulesVars.getOrDefault("WebAuthAutoUseIntentDataOnFirstTry", "false"));
		if (bUseWebAuthAutoUseIntentDataOnFirstTry)
		{ 
			if (preferences.getBoolean("WebAuthAwaitingResponse", false))
			{
				Intent webAuthIntent = getIntent();
				if (webAuthIntent != null && ACTION_VIEW.equals(webAuthIntent.getAction()))
				{
					Uri data = webAuthIntent.getData();
					if (data != null)
					{
						preferences.edit().putBoolean("WebAuthAwaitingResponse", false).apply();
						handleAuthSessionResponse(data.toString());
						return;
					}
				}
			}
		}
		
		Intent intent = new Intent(Intent.ACTION_VIEW);
		// From androidx source code
		]]>
		</insert>
		<if condition="bWebAuthSetLaunchAuthTab">
			<true>
				<insertValue value="		intent.putExtra(&quot;androidx.browser.auth.extra.LAUNCH_AUTH_TAB&quot;, true);"/>
				<insertNewline/>
			</true>
		</if>
		<insert>
			<![CDATA[
		// A null EXTRA_SESSION is interpreted as a CustomTabs intent by browsers that don't support AuthTab
		Bundle bundle = new Bundle();
		bundle.putBinder(CustomTabsIntent.EXTRA_SESSION, null);
		intent.putExtras(bundle);
		intent.setData(Uri.parse(uri));
		try
		{
			preferences.edit().putBoolean("WebAuthAwaitingResponse", true).apply();
			WebAuthAwaitingResponse = true;
			startActivity(intent);
		}
		catch (Throwable e)
		{
			// Browser not installed... for now throwing a login error is better than crashing
			Log.debug("Browser not installed");
			preferences.edit().putBoolean("WebAuthAwaitingResponse", false).apply();
			WebAuthAwaitingResponse = false;
			handleAuthSessionResponse("");
		}
	}

	private void handleAuthenticationUri(Uri data)
	{
		getSharedPreferences("WebAuth", Context.MODE_PRIVATE).edit().putBoolean("WebAuthAwaitingResponse", false).apply();
		WebAuthAwaitingResponse = false;

		if (data != null)
		{
			handleAuthSessionResponse(data.toString());
		}
		else
		{
			handleAuthSessionResponse("");
		}
	}

	@Override
	public void onWindowFocusChanged(boolean hasFocus)
	{
		super.onWindowFocusChanged(hasFocus);
		
		if (hasFocus && WebAuthAwaitingResponse)
		{
			if (!bIntentModified && !IsActivityPaused())
			{
				handleAuthenticationUri(null);
			}
		}
	}
	
	public void AndroidThunkJava_WebAuthStoreCredentials(String Id, String Token, String EnvironmentName)
	{
		if (!Token.isEmpty() && !Id.isEmpty())
		{
			SharedPreferences preferences = WebAuthGetSharedPrefs(EnvironmentName);
			SharedPreferences.Editor editor = preferences.edit();
			editor.putString("MCP_DeviceToken", Token);
			editor.putString("MCP_DeviceId", Id);
			editor.apply();
		}
	}

	public void AndroidThunkJava_WebAuthClearCredentials(String EnvironmentName)
	{
		SharedPreferences preferences = WebAuthGetSharedPrefs(EnvironmentName);
		SharedPreferences.Editor editor = preferences.edit();
		editor.putString("MCP_DeviceToken", null);
		editor.putString("MCP_DeviceId", null);
		editor.apply();
	}

	public String AndroidThunkJava_WebAuthGetId(String EnvironmentName)
	{
		SharedPreferences preferences = WebAuthGetSharedPrefs(EnvironmentName);
		return preferences.getString("MCP_DeviceId", "");
	}

	public String AndroidThunkJava_WebAuthGetToken(String EnvironmentName)
	{
		SharedPreferences preferences = WebAuthGetSharedPrefs(EnvironmentName);
		return preferences.getString("MCP_DeviceToken", "");
	}

	private SharedPreferences WebAuthGetSharedPrefs(String EnvironmentName)
	{
		return getSharedPreferences("McpSettings_" + EnvironmentName, Context.MODE_PRIVATE);
	}

	private void WebAuthClearLoginCookies(String Prefix, String Scheme, String Domain, String Path, String DomainPrefix)
	{
		String LoginDomain = Scheme + "://" + DomainPrefix + Domain;
		String Cookies = null;
		CookieManager cookieManager = CookieManager.getInstance();

		// first find and remove all the base path cookies
		Cookies = cookieManager.getCookie(LoginDomain);
		if (Cookies != null) {
			for (String CookieString : Cookies.split(";")) {
				HttpCookie Cookie = HttpCookie.parse(CookieString).get(0);
				if (Cookie.getName().startsWith(Prefix)) {
					String SetCookieString = Cookie.getName() + "=; Path=/; Max-Age=-1;";
					cookieManager.setCookie(LoginDomain, SetCookieString);
				}
			}
		}

		// find and remove all the cookies under the login path, going one path segment at a time
		String CookiePath = "";
		Uri LoginUrl = Uri.parse(LoginDomain + "/" + Path);
		for (String PathSegment : LoginUrl.getPathSegments())
		{
			CookiePath = CookiePath + "/" + PathSegment;
			Cookies = CookieManager.getInstance().getCookie(LoginDomain + CookiePath);
			if (Cookies != null) {
				for (String CookieString : Cookies.split(";")) {
					HttpCookie Cookie = HttpCookie.parse(CookieString).get(0);
					if (Cookie.getName().startsWith(Prefix)) {
						String SetCookieString = Cookie.getName() + "=; Path=" + CookiePath + "; Max-Age=-1;";
						cookieManager.setCookie(LoginDomain + CookiePath, SetCookieString);
					}
				}
			}
		}

		cookieManager.flush();
		Log.debug("Cleared cookies for " + LoginDomain);
	}

	public void AndroidThunkJava_WebAuthDeleteLoginCookies(String Prefix, String Scheme, String Domain, String Path)
	{
		WebAuthClearLoginCookies(Prefix, Scheme, Domain, Path, "");

		if (!Domain.startsWith(".")) {
			// also check login url with domain prefixed with .
			WebAuthClearLoginCookies(Prefix, Scheme, Domain, Path, ".");
		}
		if (!Domain.startsWith("www")) {
			// also check login url with domain prefixed with www.
			WebAuthClearLoginCookies(Prefix, Scheme, Domain, Path, "www.");
		}

		// Look for any remaining cookies
//		String LoginDomain = Scheme + "://" + Domain;
//		CookieManager cookieManager = CookieManager.getInstance();
//		String Cookies = cookieManager.getCookie(LoginDomain);		// + Path
//		if (Cookies != null) {
//			StringBuilder CookieNames = new StringBuilder();
//			for (String CookieString : Cookies.split(";")) {
//				HttpCookie Cookie = HttpCookie.parse(CookieString).get(0);
//				CookieNames.append(" ");
//				CookieNames.append(Cookie.getName());
//			}
//			Log.debug(LoginDomain + LoginUrl.getPath() + " cookies:" + CookieNames.toString());
//		}
	}
		]]>
		</insert>
	</gameActivityClassAdditions>

</root>
