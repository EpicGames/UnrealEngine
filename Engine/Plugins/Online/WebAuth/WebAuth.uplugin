{
	"FileVersion" : 3,

	"FriendlyName" : "Web Authentication Plugin",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to Web Authenticated Sessions.",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

	"Modules": [
		{
			"Name": "WebAuth",
			"Type": "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
				"IOS",
				"Android"
			],
			"TargetDenyList": [ "Server" ]
		}
	]
}
